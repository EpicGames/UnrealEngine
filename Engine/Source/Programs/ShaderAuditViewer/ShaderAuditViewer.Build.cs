// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ShaderAuditViewer : ModuleRules
{
	public ShaderAuditViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"InputCore",
				"Projects",
				"RenderCore",
				"ShaderAuditCore",
				"Slate",
				"SlateCore",
				"StandaloneRenderer",
				"TreeMap",
				"ToolWidgets",
			}
		);

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"SlateReflector",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"SlateReflector",
				}
			);
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.Add("UnixCommonStartup");
		}
	}
}
