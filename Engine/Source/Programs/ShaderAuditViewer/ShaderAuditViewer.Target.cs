// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class ShaderAuditViewerTarget : TargetRules
{
	public ShaderAuditViewerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "ShaderAuditViewer";	
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		// Standalone Slate application — no engine, no editor
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;
		bBuildDeveloperTools = true;
		bBuildWithEditorOnlyData = true; // Required for loading DevelopmentAssetRegistry.bin
		bCompileICU = false;
		bIsBuildingConsoleApplication = true;
		bHasExports = false;
	}
}
